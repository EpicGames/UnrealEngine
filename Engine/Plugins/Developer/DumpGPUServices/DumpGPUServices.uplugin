{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Dump GPU Services",
	"Description": "Implements automatic upload services for the DumpGPU command.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DumpGPUServices",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ],
			"TargetConfigurationDenyList": [ "Shipping" ]
		}
	]
}