// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class DumpGPUServices : ModuleRules
	{
		public DumpGPUServices(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RenderCore",
					"Json",
					"HTTP",
				}
				);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
					"Slate",
					}
					);
			}

			// Engine's NotForLicensees/ configuration are always striped out of staged builds by default. To get DumpGPU services available on all
			// internal projects automatically on staged builds,  read the engine wide NotForLicensees' default URL pattern at compile time.
			ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, null, Target.Platform);
			string EngineDefaultUploadURLPattern = "";
			if (Config.GetString("Rendering.DumpGPUServices", "EngineDefaultUploadURLPattern", out EngineDefaultUploadURLPattern))
			{
				PrivateDefinitions.Add("DUMPGPU_SERVICES_DEFAULT_URL_PATTERN=\"" + EngineDefaultUploadURLPattern + "\"");
			}
		}
	}
}
