// Copyright Epic Games, Inc. All Rights Reserved.

#include "DumpGPU.h"
#include "DumpGPUServices.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/PlatformFile.h"
#include "Misc/App.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/Paths.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"

#if WITH_EDITOR
	#include "Framework/Notifications/NotificationManager.h"
#endif

class FDumpGPUHTTPUpload;


TSharedPtr<FJsonObject> LoadJsonFile(const FString& JsonPath)
{
	TSharedPtr<FJsonObject> JsonObject;

	FString JsonContent;
	if (!FFileHelper::LoadFileToString(JsonContent, *JsonPath))
	{
		return JsonObject;
	}

	auto JsonReader = TJsonReaderFactory<>::Create(JsonContent);
	FJsonSerializer::Deserialize(JsonReader, JsonObject);
	return JsonObject;
}

class FDumpGPUHTTPUploadProvider : public IDumpGPUUploadServiceProvider
{
public:
	FDumpGPUHTTPUploadProvider(const FString& InUploadURLPattern)
		: UploadURLPattern(InUploadURLPattern)
	{ }

	virtual void UploadDump(const FDumpParameters& Parameters) override;

	~FDumpGPUHTTPUploadProvider();

private:
	const FString UploadURLPattern;
	TArray<FDumpGPUHTTPUpload*, TInlineAllocator<1>> PendingUploads;

	friend class FDumpGPUHTTPUpload;
};

IDumpGPUUploadServiceProvider* CreateHTTPUploadProvider(const FString& UploadURLPattern)
{
	return new FDumpGPUHTTPUploadProvider(UploadURLPattern);
}

class FDumpGPUHTTPUpload
{
public:
	static constexpr int32 kOverlappedRequests = 4;

	FDumpGPUHTTPUpload(FDumpGPUHTTPUploadProvider* InProvider, const IDumpGPUUploadServiceProvider::FDumpParameters& InDumpParamters, const FString& InDumpUploadURL)
		: DumpParamters(InDumpParamters)
		, DumpUploadURL(InDumpUploadURL)
		, Provider(InProvider)
	{ }

	void Start()
	{
		check(IsInGameThread());

		IPlatformFile::GetPlatformPhysical().FindFilesRecursively(Files, *DumpParamters.LocalPath, nullptr);

#if WITH_EDITOR
		ProgressHandle = FSlateNotificationManager::Get().StartProgressNotification(NSLOCTEXT("DumpGPUSerices", "UploadingGPUDump", "Uploading GPU dump"), Files.Num());
#endif

		for (int32 FileId = 0; FileId < kOverlappedRequests; FileId++)
		{
			StartNextFileUpload();
		}
	}

	void AbortDueToError()
	{
		check(IsInGameThread());

		if (!bAbort)
		{
			#if WITH_EDITOR
			if (ProgressHandle.IsValid())
			{
				FSlateNotificationManager::Get().CancelProgressNotification(ProgressHandle);
			}
			#endif
			bAbort = true;
		}

		if (PendingHttpRequests.Num() == 0)
		{
			return Finish();
		}
	}

private:
	void StartUploadFile(int32 FileId)
	{
		check(IsInGameThread());
		check(FileId < Files.Num());

		FString RelativeFilePath = Files[FileId];
		FPaths::MakePathRelativeTo(RelativeFilePath, *DumpParamters.LocalPath);

		FString FilePath = DumpParamters.LocalPath / RelativeFilePath;

		TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
		PendingHttpRequests.Add(HttpRequest);

		HttpRequest->SetURL(DumpUploadURL / RelativeFilePath);
		HttpRequest->SetVerb(TEXT("PUT"));
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/octet-stream"));

		// If the dump service parameters, upload the dump service parameters used for the upload that might have different compression settings.
		if (RelativeFilePath == IDumpGPUUploadServiceProvider::FDumpParameters::kServiceFileName)
		{
			FString ServiceParameters = DumpParamters.DumpServiceParametersFileContent();
			HttpRequest->SetContentAsString(ServiceParameters);
			return ProcessHttpRequest(HttpRequest, RelativeFilePath);
		}

		// If no compression is needed on that file, just process the HTTP request immediately.
		if (DumpParamters.CompressionName.IsNone() || !DumpParamters.CompressionFiles.IsMatch(RelativeFilePath))
		{
			if (!HttpRequest->SetContentAsStreamedFile(FilePath))
			{
				return AbortDueToError();
			}

			return ProcessHttpRequest(HttpRequest, RelativeFilePath);
		}

		// Load and compress in a background task
		FFunctionGraphTask::CreateAndDispatchWhenReady([this, RelativeFilePath, HttpRequest]()
		{
			FString FilePath = this->DumpParamters.LocalPath / RelativeFilePath;

			bool bSuccess = false;
			TArray<uint8> CompressedArray;

			TArray<uint8> UncompressedData;
			if (FFileHelper::LoadFileToArray(UncompressedData, *FilePath))
			{
				const int32 UncompressedSize = UncompressedData.Num();
				const int32 MaxCompressedSize = FCompression::CompressMemoryBound(DumpParamters.CompressionName, UncompressedSize, COMPRESS_BiasSize);

				CompressedArray.SetNumUninitialized(MaxCompressedSize);
				int32 CompressedSize = MaxCompressedSize;
				bSuccess = FCompression::CompressMemory(
					DumpParamters.CompressionName,
					/* out */ CompressedArray.GetData(), /* out */ CompressedSize,
					UncompressedData.GetData(), UncompressedSize,
					COMPRESS_BiasSize);

				UncompressedData.Empty();

				if (bSuccess)
				{
					check(CompressedSize <= MaxCompressedSize);
					CompressedArray.SetNum(CompressedSize, EAllowShrinking::No);
					UE_LOGF(LogDumpGPUServices, Verbose, "Compress %ls completed", *RelativeFilePath);
				}
				else
				{
					UE_LOGF(LogDumpGPUServices, Warning, "Failed to compress %ls", *RelativeFilePath);
				}
			}
			else
			{
				UE_LOGF(LogDumpGPUServices, Warning, "Failed to load %ls", *RelativeFilePath);
			}

			if (bSuccess)
			{
				HttpRequest->SetContent(MoveTemp(CompressedArray));

				// Queue game thread task to issue HTTP request
				FFunctionGraphTask::CreateAndDispatchWhenReady([this, RelativeFilePath, HttpRequest]()
				{
					this->ProcessHttpRequest(HttpRequest, RelativeFilePath);
				}, TStatId(), nullptr, ENamedThreads::GameThread);
			}
			else
			{
				// Queue game thread task to issue error
				FFunctionGraphTask::CreateAndDispatchWhenReady([this]()
				{
					this->AbortDueToError();
				}, TStatId(), nullptr, ENamedThreads::GameThread);
			}
		}, TStatId(), nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);
	}

	void ProcessHttpRequest(TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> HttpRequest, const FString & RelativeFilePath)
	{
		check(IsInGameThread());

		HttpRequest->OnProcessRequestComplete().BindLambda([this, RelativeFilePath](FHttpRequestPtr HttpRequestPtr, FHttpResponsePtr HttpResponse, bool bSuccess)
		{
			check(IsInGameThread());

			EHttpRequestStatus::Type RequestStatus = HttpRequestPtr->GetStatus();
			int32 ResponseCode = HttpResponse ? HttpResponse->GetResponseCode() : 0;

			// Remove the request from in-flight requests
			PendingHttpRequests.Remove(HttpRequestPtr);

			if (RequestStatus == EHttpRequestStatus::Succeeded && ResponseCode == 201)
			{
				UE_LOGF(LogDumpGPUServices, Verbose, "Upload %ls completed", *RelativeFilePath);

				FilesCompleted++;

				#if WITH_EDITOR
				if (!bAbort && ProgressHandle.IsValid())
				{
					FSlateNotificationManager::Get().UpdateProgressNotification(ProgressHandle, FilesCompleted);
				}
				#endif

				if ((FilesCompleted % 20) == 0)
				{
					UE_LOGF(LogDumpGPUServices, Display, "Uploaded %d out of %d files", FilesCompleted, Files.Num());
				}

				StartNextFileUpload();
			}
			else
			{
				UE_LOGF(LogDumpGPUServices, Warning, "Upload %ls failed", *RelativeFilePath);
				AbortDueToError();
			}
		});

		HttpRequest->ProcessRequest();
	}

	void StartNextFileUpload()
	{
		check(IsInGameThread());

		if (bAbort)
		{
			return AbortDueToError();
		}

		if (NextFileToUpload < Files.Num())
		{
			StartUploadFile(NextFileToUpload);
			NextFileToUpload++;
		}

		if (PendingHttpRequests.Num() == 0)
		{
			return Finish();
		}
	}
	
	void Finish()
	{
		check(IsInGameThread());
		check(PendingHttpRequests.Num() == 0);

		FString AbsDumpingDirectoryPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*DumpParamters.LocalPath);
		if (bAbort == false)
		{
			IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*DumpParamters.LocalPath);
			UE_LOGF(LogDumpGPUServices, Display, "DumpGPU HTTP Upload to %ls completed", *DumpUploadURL);
			UE_LOGF(LogDumpGPUServices, Display, "%ls has been deleted", *AbsDumpingDirectoryPath);

			#if WITH_EDITOR && PLATFORM_DESKTOP
				FPlatformProcess::LaunchURL(*DumpUploadURL, nullptr, nullptr);
			#endif
		}
		else
		{
			UE_LOGF(LogDumpGPUServices, Warning, "DumpGPU HTTP Upload to %ls failed", *DumpUploadURL);
			UE_LOGF(LogDumpGPUServices, Display, "%ls is still available on disk", *AbsDumpingDirectoryPath);
		}

		if (Provider->PendingUploads.Contains(this))
		{
			Provider->PendingUploads.Remove(this);
			if (Provider->PendingUploads.Num() > 0)
			{
				Provider->PendingUploads[0]->Start();
			}
		}

		delete this;
	}

	const IDumpGPUUploadServiceProvider::FDumpParameters DumpParamters;
	const FString DumpUploadURL;
	FDumpGPUHTTPUploadProvider* const Provider;
	int32 FilesCompleted = 0;
	#if WITH_EDITOR
		FProgressNotificationHandle ProgressHandle;
	#endif

	TArray<FString> Files;

	int32 NextFileToUpload = 0;
	bool bAbort = false;

	TArray<TSharedPtr<IHttpRequest, ESPMode::ThreadSafe>, TInlineAllocator<kOverlappedRequests>> PendingHttpRequests;
}; // FDumpGPUHTTPUpload


void FDumpGPUHTTPUploadProvider::UploadDump(const FDumpParameters& DumpParameters)
{
	check(IsInGameThread());

	FString UploadURL;
	{
		FString Project = FApp::GetProjectName();
		FString Platform = FPlatformProperties::IniPlatformName();

		UploadURL = UploadURLPattern;
		UploadURL.ReplaceInline(TEXT("[Project]"), *Project);
		UploadURL.ReplaceInline(TEXT("[Platform]"), *Platform);
		UploadURL.ReplaceInline(TEXT("[DumpTime]"), *DumpParameters.Time);
		UploadURL.ReplaceInline(TEXT("[DumpType]"), *DumpParameters.Type);
	}

	FDumpGPUHTTPUpload* PendingUpload = new FDumpGPUHTTPUpload(this, DumpParameters, UploadURL);
	PendingUploads.Add(PendingUpload);
	if (PendingUploads.Num() == 1)
	{
		PendingUploads[0]->Start();
	}
}

FDumpGPUHTTPUploadProvider::~FDumpGPUHTTPUploadProvider()
{
	check(IsInGameThread());

	if (PendingUploads.Num() > 0)
	{
		for (int32 i = 1; i < PendingUploads.Num(); i++)
		{
			delete PendingUploads[i];
		}
		PendingUploads.SetNum(1);

		PendingUploads[0]->AbortDueToError();
		ensure(PendingUploads.Num() == 0);
	}
}
