// Copyright Epic Games, Inc. All Rights Reserved.

#include "DumpGPUServices.h"
#include "DumpGPU.h"
#include "Misc/ConfigCacheIni.h"
#include "IDumpGPUServices.h"
#include "DumpGPU.h"

class FDumpGPUServices : public IDumpGPUServices
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	IDumpGPUUploadServiceProvider* UploadProvider = nullptr;
};

IMPLEMENT_MODULE( FDumpGPUServices, DumpGPUServices )

DEFINE_LOG_CATEGORY(LogDumpGPUServices);

IDumpGPUUploadServiceProvider* CreateHTTPUploadProvider(const FString& UploadURLPattern);


void FDumpGPUServices::StartupModule()
{
	if (IDumpGPUUploadServiceProvider::GProvider)
	{
		return;
	}

	FString UploadURLPattern;
	GConfig->GetString(TEXT("Rendering.DumpGPUServices"), TEXT("UploadURLPattern"), UploadURLPattern, GEngineIni);

	// If the project has not set UploadURLPattern, uses the engine default that maybe set in a NotForLicensees/ config.
	#if defined(DUMPGPU_SERVICES_DEFAULT_URL_PATTERN)
	if (UploadURLPattern.IsEmpty())
	{
		UploadURLPattern = TEXT(DUMPGPU_SERVICES_DEFAULT_URL_PATTERN);
	}
	#endif

	if (UploadURLPattern.StartsWith(TEXT("http://")))
	{
		UploadProvider = CreateHTTPUploadProvider(UploadURLPattern);
	}

	if (UploadProvider)
	{
		IDumpGPUUploadServiceProvider::GProvider = UploadProvider;
		UE_LOGF(LogDumpGPUServices, Display, "DumpGPU upload service set up with %ls", *UploadURLPattern);
	}
}

void FDumpGPUServices::ShutdownModule()
{
	if (!UploadProvider)
	{
		return;
	}

	if (IDumpGPUUploadServiceProvider::GProvider == UploadProvider)
	{
		IDumpGPUUploadServiceProvider::GProvider = nullptr;
	}

	delete UploadProvider;
}
